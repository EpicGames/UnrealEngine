{
	"FileVersion": 3,
	"FriendlyName": "Launcher Chunk Installer",
	"Version": 1,
	"VersionName": "1.0",
	"Description": "Chunk installer module that hooks into launcher",
	"Category": "Other",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"EnabledByDefault": true,
	"Modules": [
		{
			"Name": "LauncherChunkInstaller",
			"Type": "Runtime",
			"LoadingPhase": "PostConfigInit",
			"PlatformAllowList": [
				"Win64",
				"Linux",
				"LinuxArm64",
				"Mac"
			]
		}
	],
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": false,
	"IsBetaVersion": false,
	"IsExperimentalVersion": false,
	"Installed": false
}