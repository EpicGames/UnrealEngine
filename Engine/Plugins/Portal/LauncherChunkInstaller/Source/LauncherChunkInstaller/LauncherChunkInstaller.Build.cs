// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class LauncherChunkInstaller : ModuleRules
{
    public LauncherChunkInstaller(ReadOnlyTargetRules Target) : base(Target)
    {
        PrivateDependencyModuleNames.AddRange(
            new string[] 
            {
                "Core",
                "PakFile",
            }
        );

        if (Target.Platform != UnrealTargetPlatform.Linux && Target.Platform != UnrealTargetPlatform.Win64 && Target.Platform != UnrealTargetPlatform.Mac)
        {
            PrecompileForTargets = PrecompileTargetsType.None;
        }
    }
}