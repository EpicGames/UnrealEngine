// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GenericPlatform/GenericPlatformChunkInstall.h"

#define UE_API LAUNCHERCHUNKINSTALLER_API

/**
* Launcher Implementation of the platform chunk install module
**/
class FLauncherChunkInstaller : public FGenericPlatformChunkInstall
{
public:
	UE_API virtual EChunkLocation::Type GetChunkLocation(uint32 ChunkID) override;
};

#undef UE_API
